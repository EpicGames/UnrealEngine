// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Math/UnrealMathUtility.h"

// simple float ALU heavy benchmark
// no dependency on any math library

uint32 EvaluateJuliaFractalAt(float X, float Y)
{
	uint32 Ret = 0;

	float dist2 = 0;
	const float maxdist2 = 1600;
	uint32 MaxIter = 300;

	// pretty Julia fractal
	float Cx = -0.73f;
	float Cy = 0.176f;

	while(dist2 <= maxdist2 && Ret < MaxIter)
	{
		float X2 = X * X - Y * Y + Cx;
		float Y2 = 2 * X * Y + Cy;

		X = X2;
		Y = Y2;

		++Ret;
		dist2 = X * X + Y * Y;
	}

	return Ret;
}


float FractalBenchmark()
{
	// scales quadratic with the time
	const int32 Extent = 256;

	int32 Count = 0;
	float Sum = 0;

	for(int y = 0; y < Extent; ++y)
	{
		for(int x = 0; x < Extent; ++x)
		{
			Sum += (float)EvaluateJuliaFractalAt((float)x / (float)Extent * 2.0f - 1.0f, (float)y / (float)Extent * 2.0f - 1.0f);
		}
	}
	
	// Aggressive math optimizations and inline expansion can optimize this benchmark's return value into a constant
	// Introduce some randomness to avoid this, noting the return value of this function is not used, rather the time it takes to complete
	return Sum / (float)(Extent * Extent) + FMath::FRand();
}
