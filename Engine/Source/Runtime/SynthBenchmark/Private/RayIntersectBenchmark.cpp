// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Math/Vector.h"
#include "Math/RandomStream.h"

// simple float ALU heavy (some branching) benchmark
// some dependency on our library (not ideal)

/** 
 *	Line Check With Triangle
 *	Algorithm based on "Fast, Minimum Storage Ray/Triangle Intersection"
 *	Returns true if the line segment does hit the triangle
 *
 * code duplication to not get a different result if the source gets optimized
 */
FORCEINLINE bool LineCheckWithTriangle(const FVector& V1, const FVector& V2, const FVector& V3, const FVector& Start, const FVector& End)
{
	const FVector Direction = End - Start;

	FVector	Edge1 = V3 - V1;
	FVector	Edge2 = V2 - V1;
	FVector	P = Direction ^ Edge2;
	double Determinant = Edge1 | P;

	if(Determinant < 0.00001)
	{
		return false;
	}

	FVector	T = Start - V1;
	double U = T | P;

	if(U < 0.0 || U > Determinant)
	{
		return false;
	}

	FVector	Q = T ^ Edge1;
	double V = Direction | Q;

	if(V < 0.0 || U + V > Determinant)
	{
		return false;
	}

	double Time = (Edge2 | Q) / Determinant;

	return Time >= 0.0;
}


float RayIntersectBenchmark()
{
	FRandomStream RandomStream(0x1234);
	
	const int32 StepCount = 200000;

	int32 HitCount = 0;

	for(int32 i = 0; i < StepCount; ++i)
	{
		FVector Tri[3] = { FVector(0.1f, 0.2f, 2.3f), FVector(2.1f, 0.2f, 0.3f), FVector(-2.1f, 0.2f, 0.3f) };

		FVector Start = RandomStream.GetUnitVector() * 3.0f;
		FVector End = RandomStream.GetUnitVector() * 3.0f;

		if(LineCheckWithTriangle(Tri[0], Tri[1], Tri[2], Start, End))
		{
			HitCount++;
		}
	}

	// to avoid getting optimized out
	return (float)HitCount / (float)StepCount;
}
