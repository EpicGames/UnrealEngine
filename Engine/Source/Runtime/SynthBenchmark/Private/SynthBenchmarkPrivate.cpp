// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "GenericPlatform/GenericPlatformSurvey.h"
#include "Misc/App.h"
#include "Modules/ModuleManager.h"
#include "SynthBenchmark.h"
#include "RHI.h"
#include "RendererInterface.h"

int32 GSynthBenchmarkGPUWarmupRounds = 3;
static FAutoConsoleVariableRef CVarSynthBenchmarkGPUWarmupRounds(
	TEXT("r.SynthBenchmark.GPUWarmupRounds"),
	GSynthBenchmarkGPUWarmupRounds,
	TEXT("The number of warmup rounds for the GPU synthetic benchmark (default 3)."),
	ECVF_Default
);

float GSynthBenchmarkGPUWarmupWorkScale = 0.15f;
static FAutoConsoleVariableRef CVarSynthBenchmarkGPUWarmupWorkScale(
	TEXT("r.SynthBenchmark.GPUWarmupWorkScale"),
	GSynthBenchmarkGPUWarmupWorkScale,
	TEXT("The workload scale applied to the GPU synthetic benchmark's warmup rounds (default 0.15f)."),
	ECVF_Default
);

float GSynthBenchmarkGPUFullTestThreshold = 1.0f;
static FAutoConsoleVariableRef CVarSynthBenchmarkGPUFullTestThreshold(
	TEXT("r.SynthBenchmark.GPUWarmupThreshold"),
	GSynthBenchmarkGPUFullTestThreshold,
	TEXT("Controls the maximum time, in seconds, used to determine whether to run the full GPU synthetic benchmark workload (default 1 second).\n")
	TEXT("If the last warmup round takes longer than this threshold, the full GPU test is not run to avoid the risk of triggering the GPU hang detection (TDR)."),
	ECVF_Default
);

float RayIntersectBenchmark();
float FractalBenchmark();

// to prevent compiler optimizations
static float GGlobalStateObject = 0.0f;

class FSynthBenchmark : public ISynthBenchmark
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	
	/** ISynthBenchmark implementation */
	virtual void Run(FSynthBenchmarkResults& InOut, bool bGPUBenchmark, float WorkScale) const override;

	virtual void GetRHIDisplay(FGPUAdpater& Out) const override;
	virtual void GetRHIInfo(FGPUAdpater& Out, FString& RHIName) const override;
};

IMPLEMENT_MODULE( FSynthBenchmark, SynthBenchmark )

void FSynthBenchmark::StartupModule()
{
	// This code will execute after your module is loaded into memory (but after global variables are initialized, of course.)
}

void FSynthBenchmark::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.
}

// @param RunCount should be around 10 but can be adjusted for precision 
// @param Function should run for about 3 ms
static FTimeSample RunBenchmark(float WorkScale, float (*Function)())
{
	float Sum = 0;

	// this test doesn't support fractional WorkScale
	uint32 RunCount = FMath::Max((int32)1, (int32)WorkScale);

	for(uint32 i = 0; i < RunCount; ++i)
	{
		FPlatformMisc::MemoryBarrier();
		// todo: compiler reorder might be an issue, use _ReadWriteBarrier or something like http://svn.openimageio.org/oiio/tags/Release-0.6.3/src/include/tbb/machine/windows_em64t.h
		const double StartTime = FPlatformTime::Seconds();

		FPlatformMisc::MemoryBarrier();

		GGlobalStateObject += Function();

		FPlatformMisc::MemoryBarrier();
		Sum += (float)(FPlatformTime::Seconds() - StartTime);
		FPlatformMisc::MemoryBarrier();
	}
	
	return FTimeSample(Sum, Sum / (float)RunCount);
}

template<int NumMethods>
void PrintGPUStats(FSynthBenchmarkStat(&GPUStats)[NumMethods], const TCHAR* EndString)
{
	for (uint32 MethodId = 0; MethodId < NumMethods; ++MethodId)
	{
		UE_LOGF(LogSynthBenchmark, Display, "         ... %.3f %ls, Confidence=%.0f%% '%ls'%ls",
			1.0f / GPUStats[MethodId].GetNormalizedTime(),
			GPUStats[MethodId].GetValueType(),
			GPUStats[MethodId].GetConfidence(),
			GPUStats[MethodId].GetDesc(),
			EndString);
	}
}

void FSynthBenchmark::Run(FSynthBenchmarkResults& InOut, bool bGPUBenchmark, float WorkScale) const
{
	check(WorkScale > 0);

	if(!bGPUBenchmark)
	{
		// run a very quick GPU benchmark (less confidence but at least we get some numbers)
		// it costs little time and we get some stats
		WorkScale = 1.0f;
	}

	const double StartTime = FPlatformTime::Seconds();

	UE_LOGF(LogSynthBenchmark, Display, "FSynthBenchmark (V0.95):  requested WorkScale=%.2f", WorkScale);
	UE_LOGF(LogSynthBenchmark, Display, "===============");
	
#if UE_BUILD_DEBUG
	UE_LOGF(LogSynthBenchmark, Display, "         Note: Values are not trustable because this is a DEBUG build!");
#endif
	
	UE_LOGF(LogSynthBenchmark, Display, "Main Processor:");

	// developer machine: Intel Xeon E5-2660 2.2GHz
	// divided by the actual value on a developer machine to normalize the results
	// Index should be around 100 +-4 on developer machine in a development build (should be the same in shipping)

	InOut.CPUStats[0] = FSynthBenchmarkStat(TEXT("RayIntersect"), 0.02561f, TEXT("s/Run"), 1.f);
	InOut.CPUStats[0].SetMeasuredTime(RunBenchmark(WorkScale, RayIntersectBenchmark));

	InOut.CPUStats[1] = FSynthBenchmarkStat(TEXT("Fractal"), 0.0286f, TEXT("s/Run"), 1.5f);
	InOut.CPUStats[1].SetMeasuredTime(RunBenchmark(WorkScale, FractalBenchmark));

	for(uint32 i = 0; i < UE_ARRAY_COUNT(InOut.CPUStats); ++i)
	{
		UE_LOGF(LogSynthBenchmark, Display, "         ... %f %ls '%ls'", InOut.CPUStats[i].GetNormalizedTime(), InOut.CPUStats[i].GetValueType(), InOut.CPUStats[i].GetDesc());
	}

	UE_LOGF(LogSynthBenchmark, Display, "");

	bool bAppIs64Bit = (sizeof(void*) == 8);

	UE_LOGF(LogSynthBenchmark, Display, "  CompiledTarget_x_Bits: %ls", bAppIs64Bit ? TEXT("64") : TEXT("32"));
	UE_LOGF(LogSynthBenchmark, Display, "  UE_BUILD_SHIPPING: %d", UE_BUILD_SHIPPING);
	UE_LOGF(LogSynthBenchmark, Display, "  UE_BUILD_TEST: %d", UE_BUILD_TEST);
	UE_LOGF(LogSynthBenchmark, Display, "  UE_BUILD_DEBUG: %d", UE_BUILD_DEBUG);

	UE_LOGF(LogSynthBenchmark, Display, "  TotalPhysicalGBRam: %d", FPlatformMemory::GetPhysicalGBRam());
	UE_LOGF(LogSynthBenchmark, Display, "  NumberOfCores (physical): %d", FPlatformMisc::NumberOfCores());
	UE_LOGF(LogSynthBenchmark, Display, "  NumberOfCores (logical): %d", FPlatformMisc::NumberOfCoresIncludingHyperthreads());

	for (uint32 MethodId = 0; MethodId < UE_ARRAY_COUNT(InOut.CPUStats); ++MethodId)
	{
		UE_LOGF(LogSynthBenchmark, Display, "  CPU Perf Index %d: %.1f (weight %.2f)", MethodId, InOut.CPUStats[MethodId].ComputePerfIndex(), InOut.CPUStats[MethodId].GetWeight());
	}
	
	// separator line
	UE_LOGF(LogSynthBenchmark, Display, " ");

	UE_LOGF(LogSynthBenchmark, Display, "Graphics:");

	UE_LOGF(LogSynthBenchmark, Display, "  Adapter Name: '%ls'", *GRHIAdapterName);
	UE_LOGF(LogSynthBenchmark, Display, "  (On Optimus the name might be wrong, memory should be ok)");
	UE_LOGF(LogSynthBenchmark, Display, "  Vendor Id: 0x%X", GRHIVendorId);
	UE_LOGF(LogSynthBenchmark, Display, "  Device Id: 0x%X", GRHIDeviceId);
	UE_LOGF(LogSynthBenchmark, Display, "  Device Revision: 0x%X", GRHIDeviceRevision);

	if (FApp::CanEverRender())
	{
		{
			FTextureMemoryStats Stats;

			if (GDynamicRHI)
			{
				RHIGetTextureMemoryStats(Stats);
			}

			if (Stats.AreHardwareStatsValid())
			{
				UE_LOG(LogSynthBenchmark, Display, TEXT("  GPU Memory: %" INT64_FMT "/%" INT64_FMT "/%" INT64_FMT " MB"),
					FMath::DivideAndRoundUp(Stats.DedicatedVideoMemory, (int64)(1024 * 1024)),
					FMath::DivideAndRoundUp(Stats.DedicatedSystemMemory, (int64)(1024 * 1024)),
					FMath::DivideAndRoundUp(Stats.SharedSystemMemory, (int64)(1024 * 1024)));
			}
		}

		float GPUTime = 0.0f;

		// not always done - cost some time.
		if (bGPUBenchmark && FModuleManager::Get().IsModuleLoaded(TEXT("Renderer")))
		{
			IRendererModule& RendererModule = FModuleManager::GetModuleChecked<IRendererModule>(TEXT("Renderer"));

			// Do a few rounds for warmup.
			for (int32 Round = 0; Round < GSynthBenchmarkGPUWarmupRounds; ++Round)
			{
				RendererModule.GPUBenchmark(InOut, WorkScale * GSynthBenchmarkGPUWarmupWorkScale);
				GPUTime = InOut.ComputeTotalGPUTime();
				UE_LOGF(LogSynthBenchmark, Display, "  GPU warmup (round %d) test: %.2fs", Round + 1, GPUTime);
				PrintGPUStats(InOut.GPUStats, TEXT(" (likely to be inaccurate)"));
			}

			// If even after a few rounds of warmup we're still taking a long time just doing a portion
			// of the work, don't risk a TDR doing the full test.
			if (GPUTime < GSynthBenchmarkGPUFullTestThreshold)
			{
				RendererModule.GPUBenchmark(InOut, WorkScale);
				GPUTime = InOut.ComputeTotalGPUTime();
				UE_LOGF(LogSynthBenchmark, Display, "  GPU final test: %.2fs", GPUTime);
				PrintGPUStats(InOut.GPUStats, TEXT(""));
			}

			if (GPUTime > 0.0f)
			{
				UE_LOGF(LogSynthBenchmark, Display, "  GPU Final Results:");
				PrintGPUStats(InOut.GPUStats, TEXT(""));
				UE_LOGF(LogSynthBenchmark, Display, "");

				for (uint32 MethodId = 0; MethodId < UE_ARRAY_COUNT(InOut.GPUStats); ++MethodId)
				{
					UE_LOGF(LogSynthBenchmark, Display, "  GPU Perf Index %d: %.1f (weight %.2f)", MethodId, InOut.GPUStats[MethodId].ComputePerfIndex(), InOut.GPUStats[MethodId].GetWeight());
				}
			}
		}

		if (GPUTime > 0.0f)
		{
			UE_LOGF(LogSynthBenchmark, Display, "  GPUIndex: %.1f", InOut.ComputeGPUPerfIndex());
		}
	}

	UE_LOGF(LogSynthBenchmark, Display, "  CPUIndex: %.1f", InOut.ComputeCPUPerfIndex());
	UE_LOGF(LogSynthBenchmark, Display, "");
	UE_LOGF(LogSynthBenchmark, Display, "         ... Total Time: %f sec",  (float)(FPlatformTime::Seconds() - StartTime));
}

// could be moved to a more central place, from FWindowsPlatformSurvey::WriteFStringToResults
static void WriteFStringToResults(TCHAR* OutBuffer, const FString& InString)
{
	FMemory::Memset( OutBuffer, 0, sizeof(TCHAR) * FHardwareSurveyResults::MaxStringLength );
	TCHAR* Cursor = OutBuffer;
	for (int32 i = 0; i < FMath::Min(InString.Len(), FHardwareSurveyResults::MaxStringLength - 1); i++)
	{
		*Cursor++ = InString[i];
	}
}

void FSynthBenchmark::GetRHIDisplay(FGPUAdpater& Out) const
{
	WriteFStringToResults(Out.AdapterName, GRHIAdapterName);
	WriteFStringToResults(Out.AdapterInternalDriverVersion, GRHIAdapterInternalDriverVersion);
	WriteFStringToResults(Out.AdapterUserDriverVersion, GRHIAdapterUserDriverVersion);
	WriteFStringToResults(Out.AdapterDriverDate, GRHIAdapterDriverDate);
}

void FSynthBenchmark::GetRHIInfo(FGPUAdpater& Out, FString& RHIName) const
{
	GetRHIDisplay(Out);

	if (GDynamicRHI)
	{
		FTextureMemoryStats TextureMemStats;
		RHIGetTextureMemoryStats(TextureMemStats);
		WriteFStringToResults(Out.AdapterDedicatedMemoryMB, FString::FromInt(static_cast<int32>(TextureMemStats.DedicatedVideoMemory / (1024*1024))));

		RHIName = GDynamicRHI->GetName();
	}
}
