// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

struct FGPUAdpater;
struct FSynthBenchmarkResults;

/**
 * The public interface to this module
 */
class ISynthBenchmark : public IModuleInterface
{

public:

	// @param >0, WorkScale 10 for normal precision and runtime of less than a second
	virtual void Run(FSynthBenchmarkResults& Out, bool bGPUBenchmark = true, float WorkScale = 10.0f) const = 0;

	// could be moved out of SynthBenchmark, it only requires the RHI, only returns valid data after the RHI started up
	virtual void GetRHIDisplay(FGPUAdpater& Out) const = 0;
	virtual void GetRHIInfo(FGPUAdpater& Out, FString& RHIName) const = 0;

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline ISynthBenchmark& Get()
	{
		return FModuleManager::LoadModuleChecked< ISynthBenchmark >( "SynthBenchmark" );
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded( "SynthBenchmark" );
	}
};

