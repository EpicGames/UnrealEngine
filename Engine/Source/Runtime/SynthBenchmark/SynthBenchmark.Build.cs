// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class SynthBenchmark : ModuleRules
	{
		public SynthBenchmark(ReadOnlyTargetRules Target) : base(Target)
		{
			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"RenderCore",
					"RHI",
					// ... add other public dependencies that you statically link with here ...
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"ApplicationCore"
					// ... add private dependencies that you statically link with here ...
				}
			);

			DynamicallyLoadedModuleNames.AddRange(
				new string[]
				{
					// ... add any modules that your module loads dynamically here ...
				}
			);
		}
	}
}
